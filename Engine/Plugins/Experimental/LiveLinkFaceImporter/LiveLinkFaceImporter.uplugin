{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "LiveLinkFaceImporter",
	"Description": "Imports CSV recordings from the Live Link Face app.",
	"Category": "Other",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://www.epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": true,
	"IsBetaVersion": true,
	"IsExperimentalVersion": false,
	"Installed": false,
	"Modules": [
		{
			"Name": "LiveLinkFaceImporter",
			"Type": "Editor",
			"LoadingPhase": "Default"
		}
	],
	"Plugins" : [
		{
			"Name": "LiveLink",
			"Enabled": true
		}
	]
}
