// Copyright Epic Games, Inc. All Rights Reserved.

#include "LiveLinkFaceImporterFactory.h"
#include "LiveLinkFaceImporterLog.h"
#include "LevelSequence.h"
#include "Misc/FileHelper.h"
#include "Misc/QualifiedFrameTime.h"
#include "Misc/Timecode.h"
#include "MovieScene.h"
#include "MovieScene/MovieSceneLiveLinkTrack.h"
#include "MovieScene/MovieSceneLiveLinkSection.h"
#include "Roles/LiveLinkBasicRole.h"
#include "LiveLinkPresetTypes.h"


//////////////////////////////////////////////////////////////////////////
namespace
{
	const int32 TIMECODE_COLUMN_INDEX = 0;
	const int32 BLENDSHAPECOUNT_COLUMN_INDEX = 1;

	const FString TIMECODE_COLUMN_NAME = TEXT("Timecode");
	const FString BLENDSHAPECOUNT_COLUMN_NAME = TEXT("BlendShapeCount");
}


//////////////////////////////////////////////////////////////////////////
// ULiveLinkFaceImporterFactory

ULiveLinkFaceImporterFactory::ULiveLinkFaceImporterFactory(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bCreateNew = false;
	SupportedClass = ULevelSequence::StaticClass();

	bEditorImport = true;
	bText = true;

	Formats.Add(TEXT("csv;Live Link Face recording"));
}

FText ULiveLinkFaceImporterFactory::GetToolTip() const
{
	return NSLOCTEXT("LiveLinkFaceImporter", "LiveLinkFaceImporterFactoryDescription", "CSV files saved by the Live Link Face iOS app.");
}

bool ULiveLinkFaceImporterFactory::LoadCSV(const FString& FileContent, TArray<FString>& KeyArray, TArray<FString>& LineArray, FString & OutLogMessage)
{
	if (FileContent.ParseIntoArrayLines(LineArray) > 1)
	{
		// At least 2 lines in the CSV are needed : One which is the column names (keys) and at least one line
		// which contains values.
		if (FileContent.ParseIntoArrayLines(LineArray) < 2)
		{
			OutLogMessage = TEXT("The input CSV file contained no data.");
			return false;
		}

		// The first line (the column names / keys) must have at least 3 fields : timecode, shape count, and 
		// at least one value column. otherwise there is no data to import.
		if (LineArray[0].ParseIntoArray(KeyArray, TEXT(",")) < 3)
		{
			OutLogMessage = TEXT("The input CSV file didn't include more than 2 columns.");
			return false;
		}
		LineArray.RemoveAt(0);

		// validate that the first two entries in the KeyArray are Timecode & BlendShapeCount
		if (KeyArray[TIMECODE_COLUMN_INDEX] != TIMECODE_COLUMN_NAME)
		{
			OutLogMessage = FString::Format(TEXT("The input CSV file was poorly formatted : the first column should be \"Timecode\"."), { *TIMECODE_COLUMN_NAME });
			return false;
		}
		// validate that the first two entries in the KeyArray are Timecode & BlendShapeCount
		if (KeyArray[BLENDSHAPECOUNT_COLUMN_INDEX] != BLENDSHAPECOUNT_COLUMN_NAME)
		{
			OutLogMessage = FString::Format(TEXT("The input CSV file was poorly formatted : the second column should be \"%s\"."), { *BLENDSHAPECOUNT_COLUMN_NAME });
			return false;
		}

	}
	return true;
}

FString ULiveLinkFaceImporterFactory::CreateSubjectString(const FString& InString)
{
	// Try to determine the subject name : The filename is <Slate>_<Take>_<SubjectName>[_cal|_raw|_neutral].csv.
	FString SubjectString = InString;

	// Remove the [_cal|_raw|_neutral] suffix if it exists
	SubjectString.RemoveFromEnd("_cal");
	SubjectString.RemoveFromEnd("_raw");
	SubjectString.RemoveFromEnd("_neutral");

	{
		TArray<FString> TokenArray;
		if (SubjectString.ParseIntoArray(TokenArray, TEXT("_"), false) > 3)
		{
			// Remove the first token (the slate)
			TokenArray.RemoveAt(0);

			// Remove each token until (but including) a token that is a number : we assume it's the take number. The
			// remaining tokens will form the subject name.
			while (TokenArray.Num())
			{
				bool bIsNumeric = TokenArray[0].IsNumeric();
				TokenArray.RemoveAt(0);

				if (bIsNumeric)
					break;
			}

			if (TokenArray.Num())
			{
				SubjectString = FString::Join(TokenArray, TEXT("_"));
			}
		}
	}

	// If we've erased everything due to an unexpected filename, just use the InString again.
	return SubjectString.IsEmpty() ? InString : SubjectString;

}

/**
 * Tokenizes a time code string and returns the hours, minutes, seconds and frames as discrete values.
 *
 * @param	InTimecodeString A string representation of a timecode (hh:mm:ss:frame.zzz)
 * @param	OutHours	Hours output
 * @param	OutMinutes	Minutes output
 * @param	OutSeconds	Seconds output
 * @param	OutFrames	Fractional frames output 
 *
 * @return	Whether or not the timecode string could be tokenized correctly.
 */
bool ULiveLinkFaceImporterFactory::ParseTimecode(const FString& InTimecodeString, int32& OutHours,
	int32& OutMinutes, int32& OutSeconds, double& OutFrames)
{
	TArray<FString> TimecodeTokens;
	InTimecodeString.ParseIntoArray(TimecodeTokens, TEXT(":"));
	if (TimecodeTokens.Num() == 4)
	{
		OutHours = FCString::Atoi(*TimecodeTokens[0]);
		OutMinutes = FCString::Atoi(*TimecodeTokens[1]);
		OutSeconds = FCString::Atoi(*TimecodeTokens[2]);
		OutFrames = FCString::Atof(*TimecodeTokens[3]);
		return true;
	}
	return false;
}

/**
 * @brief Infers the frame rate of the CSV data as either 30 or 60 FPS.
 * 
 * There is no explicit metadata available within the CSV output file generated by the Live Link Face application that
 * describes the frame rate of the captured data.
 * 
 * However we know that when recording at 30 FPS the LLF app will not output timecodes with fractional frame values
 * greater than 30. With that in mind we can infer when a file was recorded at a target of 30 or 60 FPS.
 * @param InLineValuesArray The values contained within the CSV file. 
 * @param OutInferredFrameRate The inferred frame rate value to populate.
 * @return Whether or not the operation was a success. 
 */
bool ULiveLinkFaceImporterFactory::InferFrameRate(const TArray<TArray<FString>>& InLineValuesArray, int8& OutInferredFrameRate)
{
	constexpr double Minimum60FPSFrameNumber = 30.0;
	int8 InferredFrameRate = 30;
	for (const TArray<FString>& ValueArray : InLineValuesArray)
	{
		if (TIMECODE_COLUMN_INDEX < ValueArray.Num())
		{
			int32 Hours;
			int32 Minutes;
			int32 Seconds;
			double FrameNumber;
			
			if (!ParseTimecode(ValueArray[TIMECODE_COLUMN_INDEX], Hours, Minutes, Seconds, FrameNumber))
			{
				return false;
			}

			if (FrameNumber > Minimum60FPSFrameNumber)
			{
				InferredFrameRate = 60;
			}
		} 
	}

	OutInferredFrameRate = InferredFrameRate;
	return true;
}

bool ULiveLinkFaceImporterFactory::FactoryCanImport(const FString& Filename)
{
	// If the file has more than 2 lines and the first line is tokenized by commas *and* has 
	// the first two columns named correctly, assume we can import it.
	FString FileContent;
	if (FFileHelper::LoadFileToString(/*out*/ FileContent, *Filename))
	{
		TArray<FString> LineArray;
		TArray<FString> KeyArray;
		FString OutLogMessage;

		if (LoadCSV(FileContent, KeyArray, LineArray, OutLogMessage))
		{
			return true;
		}
		else
		{
			UE_LOGF(LogLiveLinkFaceImporter, VeryVerbose, "%ls", *OutLogMessage);
			return false;
		}
	}

	UE_LOGF(LogLiveLinkFaceImporter, Error, "Unable to load the file '%ls'", *Filename);
	return false;
}

UObject* ULiveLinkFaceImporterFactory::FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn)
{
	const FString FileContent = FString::ConstructFromPtrSize(Buffer, BufferEnd - Buffer);
	TArray<FString> LineArray;
	TArray<FString> KeyArray;
	FString OutLogMessage;

	// Load the CSV into Keys (the first row tokenized) and Lines (the rest of the file)
	if (LoadCSV(FileContent, KeyArray, LineArray, OutLogMessage) == false)
	{
		UE_LOGF(LogLiveLinkFaceImporter, Error, "%ls", *OutLogMessage);
		return nullptr;
	}

	UE_LOGF(LogLiveLinkFaceImporter, Log, "Importing CSV file.");

	// Create the subject name from the InName (the filename)
	FString SubjectString = CreateSubjectString(InName.ToString());

	UE_LOGF(LogLiveLinkFaceImporter, Verbose, "Using subject name '%ls'.", *SubjectString);

	FName SubjectName(SubjectString);
	UClass* SubjectRole = ULiveLinkBasicRole::StaticClass();

	// create the new sequence
	ULevelSequence* NewLevelSequence = NewObject<ULevelSequence>(InParent, InName, Flags | RF_Transactional);
	NewLevelSequence->Initialize();

	// save the tick + display rate as we will need them to convert from the CSV's timecode
	const FFrameRate TickResolution = NewLevelSequence->GetMovieScene()->GetTickResolution();
	
	// Add the LL track
	UMovieSceneLiveLinkTrack* LiveLinkTrack = NewLevelSequence->GetMovieScene()->AddTrack<UMovieSceneLiveLinkTrack>();
	LiveLinkTrack->SetTrackRole(SubjectRole);
	LiveLinkTrack->SetPropertyNameAndPath(SubjectName, SubjectName.ToString());

	// Create and add a section to the LL track
	UMovieSceneLiveLinkSection* MovieSceneSection = Cast<UMovieSceneLiveLinkSection>(LiveLinkTrack->CreateNewSection());

	MovieSceneSection->SetIsActive(false);
	LiveLinkTrack->AddSection(*MovieSceneSection);

	FLiveLinkSubjectPreset SubjectPreset;
	SubjectPreset.Key.Source.Invalidate();
	SubjectPreset.Key.SubjectName = SubjectName;
	SubjectPreset.Role = SubjectRole;
	SubjectPreset.bEnabled = true;

	// Add all the property names from the first row of the CSV (skipping Timecode & BlendshapeCount)
	FLiveLinkBaseStaticData BaseStaticData;
	for (const FString& Key : KeyArray)
	{
		if ((Key != TIMECODE_COLUMN_NAME) && (Key != BLENDSHAPECOUNT_COLUMN_NAME))
		{
			BaseStaticData.PropertyNames.Add(*Key);
		}
	}

	// initialize the movie scene section with the given properties from the CSV
	TSharedPtr<FLiveLinkStaticDataStruct> StaticData = MakeShared<FLiveLinkStaticDataStruct>();
	StaticData->InitializeWith(&BaseStaticData);
	MovieSceneSection->Initialize(SubjectPreset, StaticData);
	MovieSceneSection->CreateChannelProxy();

	// Tokenize each line within the CSV file
	TArray<TArray<FString>> LineValuesArray;
	LineValuesArray.Reserve(LineArray.Num());
	for (const FString& Line : LineArray)
	{
		TArray<FString> ValueArray;
		Line.ParseIntoArray(ValueArray, TEXT(","));
		LineValuesArray.Add(ValueArray);
	}

	// Infer the frame rate of the file contents in the absence of any explicit metadata.
	int8 InferredFrameRate;
	if (!InferFrameRate(LineValuesArray, InferredFrameRate))
	{
		UE_LOGF(LogLiveLinkFaceImporter, Warning, "Failed to infer frame rate from CSV file contents. Assuming 60 FPS.")
		InferredFrameRate = 60;
	}

	// Set the initial display rate of the movie scene to that inferred from the CSV file contents.
	NewLevelSequence->GetMovieScene()->SetDisplayRate(FFrameRate(InferredFrameRate, 1));
	
	int32 LineNumber = 1;
	int32 FrameCount = 0;
	bool IsFirstFrame = true;
	for (const TArray<FString>& ValueArray : LineValuesArray)
	{
		// the number of values in each line must match the keys/header in the first line
		int32 ValueCount = ValueArray.Num();
		if (ValueCount != KeyArray.Num())
		{
			UE_LOGF(LogLiveLinkFaceImporter, Error, "Line %d did not contain the correct number of columns (found %d, expected %d). Aborting!", LineNumber, ValueCount, KeyArray.Num());
			break;
		}
		
		// now build the frame data for this line
		FLiveLinkBaseFrameData FrameData;
		double SampleTime = 0.0;

		for (int32 Index = 0; Index < ValueArray.Num(); ++Index)
		{
			if (Index == BLENDSHAPECOUNT_COLUMN_INDEX)
			{
				continue;
			}
			else if (Index == TIMECODE_COLUMN_INDEX)
			{
				int32 Hours;
				int32 Minutes;
				int32 Seconds;
				double Frames;
				if (ParseTimecode(ValueArray[Index], Hours, Minutes, Seconds, Frames))
				{
					SampleTime = double(Hours * 60 * 60 + Minutes * 60 + Seconds) + Frames / static_cast<double>(InferredFrameRate);
					FTimecode Timecode = FTimecode(SampleTime, FFrameRate(InferredFrameRate,1), false);

					// use this timecode as the source if we are on the first frame
					if (IsFirstFrame)
					{
						MovieSceneSection->TimecodeSource = FMovieSceneTimecodeSource(Timecode);
						IsFirstFrame = false;
					}
				}
				else
				{
					UE_LOGF(LogLiveLinkFaceImporter, Error, "Line %d contained a malformed timecode (Should be of format hh:mm:ss:frame.zzz). Skipping!", LineNumber);
					break;
				}
			}
			else
			{
				// add a keyframe for this property
				FrameData.PropertyValues.Add(FCString::Atof(*ValueArray[Index]));
			}
		}

		// create the frame data struct and add it to the movie scene section
		FLiveLinkFrameDataStruct Frame;
		Frame.InitializeWith(&FrameData);

		MovieSceneSection->RecordFrame((SampleTime * TickResolution).FloorToFrame(), MoveTemp(Frame));

		// these are for logging purposes
		FrameCount++;
		LineNumber++;
	}

	// finish up the sequence + set the section/playback ranges

	FKeyDataOptimizationParams Params;
	Params.bAutoSetInterpolation = true;

	MovieSceneSection->FinalizeSection(false, Params);

	TOptional<TRange<FFrameNumber>> DefaultSectionLength = MovieSceneSection->GetAutoSizeRange();
	if (DefaultSectionLength.IsSet())
	{
		MovieSceneSection->SetRange(DefaultSectionLength.GetValue());
		NewLevelSequence->GetMovieScene()->SetPlaybackRange(DefaultSectionLength.GetValue());
	}
	MovieSceneSection->SetIsActive(true);

	// all done
	UE_LOGF(LogLiveLinkFaceImporter, Log, "Successfully imported %d frames.", FrameCount);

	return NewLevelSequence;
}
