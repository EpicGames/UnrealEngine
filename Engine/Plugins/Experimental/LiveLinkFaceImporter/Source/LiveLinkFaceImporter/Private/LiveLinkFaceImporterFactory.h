// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Factories/Factory.h"
#include "LiveLinkFaceImporterFactory.generated.h"

// Imports a CSV file created by the Live Link Face iOS app when recording facial animation.
UCLASS()
class ULiveLinkFaceImporterFactory : public UFactory
{
	GENERATED_UCLASS_BODY()
		
	// UFactory interface
	virtual FText GetToolTip() const override;
	virtual bool FactoryCanImport(const FString& Filename) override;
	virtual UObject* FactoryCreateText(UClass* InClass, UObject* InParent, FName InName, EObjectFlags Flags, UObject* Context, const TCHAR* Type, const TCHAR*& Buffer, const TCHAR* BufferEnd, FFeedbackContext* Warn) override;
	// End of UFactory interface

protected:
	bool LoadCSV(const FString& FileContent, TArray<FString>& KeyArray, TArray<FString>& LineArray, FString& OutLogMessage);
	FString CreateSubjectString(const FString& InName);
	bool ParseTimecode(const FString& InTimecodeString, int32& OutHours, int32& OutMinutes, int32& OutSeconds, double& OutFrames);
	bool InferFrameRate(const TArray<TArray<FString>>& InLineValuesArray, int8& OutInferredFrameRate);
};

