// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "LiveLinkFaceImporterLog.h"
#include "Modules/ModuleManager.h"
#include "Internationalization/Internationalization.h"

#define LOCTEXT_NAMESPACE "LiveLinkFaceImporter"

DEFINE_LOG_CATEGORY(LogLiveLinkFaceImporter);

//////////////////////////////////////////////////////////////////////////
// FLiveLinkFaceImporterModule

class FLiveLinkFaceImporterModule : public FDefaultModuleImpl
{
public:
	virtual void StartupModule() override
	{
	}

	virtual void ShutdownModule() override
	{
	}
};

//////////////////////////////////////////////////////////////////////////

IMPLEMENT_MODULE(FLiveLinkFaceImporterModule, LiveLinkFaceImporter);

//////////////////////////////////////////////////////////////////////////

#undef LOCTEXT_NAMESPACE
